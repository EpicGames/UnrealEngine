{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Code View",
	"Description" : "Provides an in-editor code view of game classes and structures with direct IDE accessibility",
	"Category" : "Programming",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "CodeView",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}