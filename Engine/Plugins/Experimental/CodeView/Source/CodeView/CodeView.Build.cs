// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class CodeView : ModuleRules
	{
		public CodeView(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Slate",
                    "InputCore",
					"Core",
					"CoreUObject",
					"Engine",
					"EditorFramework",
					"UnrealEd",
					"PropertyEditor",
					"DesktopPlatform",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DetailCustomizations",
					"SlateCore",
					"SourceCodeAccess"
				}
			);
		}
	}
}
