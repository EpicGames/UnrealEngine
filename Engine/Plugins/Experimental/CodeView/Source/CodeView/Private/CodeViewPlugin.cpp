// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "SCodeView.h"
#include "ISourceCodeAccessor.h"
#include "ISourceCodeAccessModule.h"

class FCodeViewPlugin : public IModuleInterface
{
public:
	// IModuleInterface implementation
	virtual void StartupModule() override
	{
		OnExtendActorDetails.AddRaw(this, &FCodeViewPlugin::AddCodeViewCategory);
	}

	virtual void ShutdownModule() override
	{
		OnExtendActorDetails.RemoveAll(this);
	}
	// End of IModuleInterface implementation

	void AddCodeViewCategory(IDetailLayoutBuilder& DetailBuilder, const FGetSelectedActors& GetSelectedActors)
	{
		ISourceCodeAccessModule* SourceCodeAccessModule = FModuleManager::GetModulePtr<ISourceCodeAccessModule>("SourceCodeAccess");
		if(SourceCodeAccessModule != nullptr && SourceCodeAccessModule->GetAccessor().CanAccessSourceCode())
		{
			TSharedRef< CodeView::SCodeView > CodeViewWidget =
				SNew( CodeView::SCodeView )
				.GetSelectedActors( GetSelectedActors );

			// Only start out expanded if we're already in "ready to populate" mode.  This is because we don't want
			// to immediately start digesting symbols as soon as the widget is visible.  Instead, when the user
			// expands the section, we'll start loading symbols.  However, this state is remembered even after
			// the widget is destroyed.
			const bool bShouldInitiallyExpand = CodeViewWidget->IsReadyToPopulate();

			DetailBuilder.EditCategory( "CodeView", NSLOCTEXT("ActorDetails", "CodeViewSection", "Code View"), ECategoryPriority::Uncommon )
				.InitiallyCollapsed( !bShouldInitiallyExpand )
				// The expansion state should not be restored
				.RestoreExpansionState( false )
				.OnExpansionChanged( FOnBooleanValueChanged::CreateSP( CodeViewWidget, &CodeView::SCodeView::OnDetailSectionExpansionChanged ) )
				.AddCustomRow( NSLOCTEXT("ActorDetails", "CodeViewSection", "Code View") )
				[
					// @todo editcode1: Width of item is too big for detail view?!
					CodeViewWidget
				];
		}
	}
};


IMPLEMENT_MODULE(FCodeViewPlugin, CodeView)


