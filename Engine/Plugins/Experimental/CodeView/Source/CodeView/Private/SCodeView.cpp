// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCodeView.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SourceCodeNavigation.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SCodeView"


namespace CodeView
{
	/** IDs for list columns */
	static const FName ColumnID_ClassesAndFunctions( "Function" );
	static const FName ColumnID_Custom( "Custom" );

	/** Kind of a hack here. We use a static setting for whether we're "expanded and ready". This is so that
	    after the user expands the section, we'll always remember that its open until the user collapses it
		in that same session */
	static bool bReadyToPopulate = false;

	namespace Helpers
	{
		/** Sorts tree items alphabetically */
		struct FTreeItemSorter
		{
			bool operator()( const FTreeItemPtr& A, const FTreeItemPtr& B	) const
			{
				const auto* ItemA = A.Get();
				const auto* ItemB = B.Get();
				if( ItemA != NULL && ItemB != NULL )
				{
					return ItemA->GetAsString() < ItemB->GetAsString();
				}
			
				return false;
			}
		};


		/** Checks to see if the specified item passes the search filter string and returns true if so */
		inline static bool PassesFilter( const FTreeItem& TreeItem, const ECustomColumnMode::Type CustomColumnMode, const FString& FilterText )
		{
			if( ( FilterText.IsEmpty() ||		// No filter text, so always passes search
				  TreeItem.GetAsString().Contains(FilterText) ) )	// Matches name?
			{
				return true;
			}

			return TreeItem.ModuleName.Contains(FilterText);
		}

		FText MakeCustomColumnComboText( const ECustomColumnMode::Type& Mode )
		{
			FText ModeName;
			switch( Mode )
			{
			case ECustomColumnMode::None:
				ModeName = LOCTEXT("CustomColumnMode_None", " - ");
				break;

			case ECustomColumnMode::ModuleName:
				ModeName = LOCTEXT("CustomColumnMode_ModuleName", "Module");
				break;

			default:
				ensure(0);
				break;
			}
			return ModeName;
		}
	}



	/** Widget that represents a row in the tree control.  Generates widgets for each column on demand. */
	class SCodeViewTreeRow
		: public SMultiColumnTableRow< FTreeItemPtr >
	{

	public:

		SLATE_BEGIN_ARGS( SCodeViewTreeRow ){}
			
			/** The widget that owns the tree.  We'll only keep a weak reference to it. */
			SLATE_ARGUMENT( TSharedPtr< SCodeView >, CodeView )

			/** The list item for this row */
			SLATE_ARGUMENT( FTreeItemPtr, Item )

		SLATE_END_ARGS()


		/** Construct function for this widget */
		void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
		{
			CodeViewWeak = InArgs._CodeView;
			Item = InArgs._Item;

			SMultiColumnTableRow< FTreeItemPtr >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
		}


		/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
		virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
		{
			// Create the widget for this item
			auto ItemWidget = CodeViewWeak.Pin()->GenerateWidgetForItemAndColumn( Item, ColumnName );

			if( ColumnName == ColumnID_ClassesAndFunctions )
			{
				// The first column gets the tree expansion arrow for this row
				return 
					SNew( SHorizontalBox )

						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew( SExpanderArrow, SharedThis(this) )
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							ItemWidget
						];
			}
			else
			{
				// Other columns just get widget content -- no expansion error needed
				return ItemWidget;
			}
		}

	private:

		/** Weak reference to the widget that owns our list */
		TWeakPtr< SCodeView > CodeViewWeak;

		/** The item associated with this row of data */
		FTreeItemPtr Item;
	};



	void SCodeView::Construct( const FArguments& InArgs )
	{
		TotalClassesAndFunctions = 0;

		// @todo editcode: Should probably save this in layout!
		// @todo editcode: Should save spacing for list view in layout
		CurrentCustomColumnMode = ECustomColumnMode::None;

		// Selected actors delegate MUST be valid
		GetSelectedActorsDelegate = InArgs._GetSelectedActors;
		check( GetSelectedActorsDelegate.IsBound() );

	
		// Build up a list of available custom column modes for Slate
		static TArray< TSharedPtr< ECustomColumnMode::Type > > CustomColumnModeOptions;
		if( CustomColumnModeOptions.Num() == 0 )
		{
			for( auto CurModeIndex = 0; CurModeIndex < ECustomColumnMode::Count; ++CurModeIndex )
			{
				CustomColumnModeOptions.Add( MakeShareable( new ECustomColumnMode::Type( ( ECustomColumnMode::Type )CurModeIndex ) ) );
			}
		}

		struct Local
		{	
			static FText MakeCustomColumnComboToolTipText(const ECustomColumnMode::Type& Mode)
			{
				FText ToolTipText;
				switch( Mode )
				{
					case ECustomColumnMode::None:
						ToolTipText = LOCTEXT("CustomColumnModeToolTip_None", "Hides all extra function info");
						break;

					case ECustomColumnMode::ModuleName:
						ToolTipText = LOCTEXT("CustomColumnModeToolTip_ModuleName", "Displays the name of the module each function resides within");
						break;

					default:
						ensure(0);
						break;
				}
				return ToolTipText;
			}


			static TSharedRef< SWidget > MakeCustomColumnComboButtonItemWidget( TSharedPtr< ECustomColumnMode::Type > Mode )
			{
				return
					SNew( STextBlock )
						.Text( Helpers::MakeCustomColumnComboText( *Mode ) )
						.ToolTipText( MakeCustomColumnComboToolTipText( *Mode ) )
					;
			}
		};

	
		// @todo editcode: We should save/load the user's column divider position!
		TSharedRef< SHeaderRow > HeaderRowWidget =
			SNew( SHeaderRow )

				/** Class/Function label column */
				+ SHeaderRow::Column( ColumnID_ClassesAndFunctions )
					.FillWidth( 0.80f )
					[
						SNew( SHorizontalBox )
							+SHorizontalBox::Slot()
								.AutoWidth()
								.Padding( 0.0f, 3.0f, 0.0f, 0.0f )
								[
									SNew( STextBlock )
										.Text( LOCTEXT("TreeColumn_FunctionLabel", "Function") )
								]
					]

				/** Customizable data column */
				+ SHeaderRow::Column( ColumnID_Custom )
					.FillWidth( 0.20f )
					[
						SNew( SHorizontalBox )
							+SHorizontalBox::Slot()
								.AutoWidth()
								.Padding( 0.0f, 3.0f, 0.0f, 0.0f )
								[
									SNew( STextBlock )
										.Text( LOCTEXT("TreeColumn_CustomColumn", "Info") )
								]

							+SHorizontalBox::Slot()
								.AutoWidth()
								.Padding( 5.0f, 0.0f, 0.0f, 0.0f )
								[
									SAssignNew( CustomColumnModeComboBox, SCustomColumnModeComboBoxType )
										.ContentPadding(FMargin(1))
										.ToolTipText( LOCTEXT("CustomColumnModeComboBox_ToolTip", "Choose what type of information to display in this column") )
										.OptionsSource( &CustomColumnModeOptions )
										.OnGenerateWidget_Static( &Local::MakeCustomColumnComboButtonItemWidget )
										.OnSelectionChanged( this, &SCodeView::OnCustomColumnModeChanged )

										// Synchronize the initial custom column mode selection
										.InitiallySelectedItem( CustomColumnModeOptions[ CurrentCustomColumnMode ] )
										[
											SNew(STextBlock)
											.Text(this, &SCodeView::GetSelectedModeText)
										]
								]
					]
			;


		ChildSlot
		[
			SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( SVerticalBox )
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding( 2.0f, 0.0f )
						[
							SAssignNew( FilterTextBoxWidget, SSearchBox )
								.ToolTipText( LOCTEXT("FilterSearchHint", "Type here to search functions") )
								.OnTextChanged( this, &SCodeView::OnFilterTextChanged )
								.OnTextCommitted( this, &SCodeView::OnFilterTextCommitted )
						]

						// The filter status line; shows how many items made it past the filter
						+SVerticalBox::Slot()
							.AutoHeight()
							.HAlign( HAlign_Center )
						[
							SNew( STextBlock )
								.Visibility( this, &SCodeView::GetFilterStatusVisibility )
								.Text( this, &SCodeView::GetFilterStatusText )
								.ColorAndOpacity( this, &SCodeView::GetFilterStatusTextColor )
						]

						+SVerticalBox::Slot()
							.AutoHeight()
							// NOTE: We enforce a fixed height to avoid weird list scrolling issues while embedded within the details view
							.MaxHeight( 200.0f )
						[
							SAssignNew( CodeTreeView, SCodeTreeView )

								// Currently, we only need single-selection for this tree
								.SelectionMode( ESelectionMode::Single )

								// Point the tree to our array of root-level items.  Whenever this changes, we'll call RequestTreeRefresh()
								.TreeItemsSource( &RootTreeItems )

								// Find out when the user selects something in the tree
								.OnSelectionChanged( this, &SCodeView::OnTreeSelectionChanged )

								// Called when the user double-clicks with LMB on an item in the list
								.OnMouseButtonDoubleClick( this, &SCodeView::OnTreeDoubleClick )

								// Called to child items for any given parent item
								.OnGetChildren( this, &SCodeView::OnGetChildrenForTree )

								// Generates the actual widget for a tree item
								.OnGenerateRow( this, &SCodeView::OnGenerateRowForTree ) 

								// Header for the tree
								.HeaderRow( HeaderRowWidget )
						]
				]
		];


		// Don't allow tool-tips over the header
		HeaderRowWidget->EnableToolTipForceField( true );

		// Register for symbol query notifications, so we can refresh our view when new symbols are digested
		FSourceCodeNavigation::AccessOnSymbolQueryFinished().AddSP( SharedThis( this ), &SCodeView::OnSymbolQueryFinished );

		// NOTE: We don't initially populate ourselves by default (bReadyToPopulate will be false, for the first
		// time this widget is used).  Instead, we wait for 'OnInspectSelectionExpanded' to be called.  For now,
		// we wait patiently for the user to expand the section.  Afterwards, we'll remember the expansion state
		// for the rest of the session.
		if( bReadyToPopulate )
		{
			Populate();
		}
	}


	SCodeView::~SCodeView()
	{
		// Unregister ourselves for symbol notification
		FSourceCodeNavigation::AccessOnSymbolQueryFinished().RemoveAll( this );
	}


	void SCodeView::Populate()
	{
		ensure( bReadyToPopulate == true );

		if( GetSelectedActorsDelegate.IsBound() )
		{
			TotalClassesAndFunctions = 0;
			RootTreeItems.Empty();
			CodeTreeView->ClearSelection();
			SignatureToTreeItemMap.Reset();
			ClassNameToTreeItemMap.Reset();

			TArray< AActor* > SelectedActors;
			for( TArray< TWeakObjectPtr< AActor > >::TConstIterator It( GetSelectedActorsDelegate.Execute() ); It ; ++It )
			{
				AActor* Actor = static_cast<AActor*>( It->Get() );
				SelectedActors.Add( Actor );
			}

			// Gather all of the functions for the currently selected actor's classes
			TArray< FSourceCodeNavigation::FEditCodeMenuClass > Classes;
			FSourceCodeNavigation::GatherFunctionsForActors( SelectedActors, FSourceCodeNavigation::EGatherMode::ClassesAndFunctions, Classes );
			for( int32 CurClassIndex = 0; CurClassIndex < Classes.Num(); ++CurClassIndex )
			{
				FSourceCodeNavigation::FEditCodeMenuClass& CurClass = Classes[ CurClassIndex ];

				// Add the class to the tree first
				TSharedPtr< FClassTreeItem > ClassTreeItemPtr( new FClassTreeItem() );
				ClassTreeItemPtr->ClassName = CurClass.Name;
				ClassTreeItemPtr->bIsCompleteList = CurClass.bIsCompleteList;
				ClassTreeItemPtr->ModuleName = CurClass.ModuleName;
				ClassTreeItemPtr->AnyFunctionSymbolName = FString();	// NOTE: Will be filled in below as we iterate over functions
				++TotalClassesAndFunctions;

				bool bAnyChildFunctionPassedFilter = false;
				TArray< FTreeItemPtr > FunctionList;

				for( TArray< FSourceCodeNavigation::FFunctionSymbolInfo >::TConstIterator CurFunctionIt( CurClass.Functions ); CurFunctionIt; ++CurFunctionIt )
				{
					const FSourceCodeNavigation::FFunctionSymbolInfo& FunctionSymbolInfo = *CurFunctionIt;
					const FString& FunctionSymbolName = FunctionSymbolInfo.SymbolName;

					// Strip off the class name if we have one
					FString FunctionOnly = FunctionSymbolName;
					const int32 ClassDelimeterPos = FunctionSymbolName.Find( TEXT( "::" ) );
					if( ClassDelimeterPos != INDEX_NONE )
					{
						// Yank off the class name, we'll store that by itself
						FunctionOnly = FunctionSymbolName.Mid( ClassDelimeterPos + 2 );
					}


					// Add the function to the tree
					TSharedPtr< FFunctionTreeItem > FunctionTreeItemPtr( new FFunctionTreeItem() );
					FunctionTreeItemPtr->FunctionName = FunctionOnly;
					FunctionTreeItemPtr->FunctionSymbolName = FunctionSymbolName;
					FunctionTreeItemPtr->ModuleName = FunctionSymbolInfo.ModuleName;
					++TotalClassesAndFunctions;

					if( ClassTreeItemPtr->AnyFunctionSymbolName.IsEmpty() )
					{
						ClassTreeItemPtr->AnyFunctionSymbolName = FunctionTreeItemPtr->FunctionSymbolName;
					}

					if( Helpers::PassesFilter( *FunctionTreeItemPtr, CurrentCustomColumnMode, FilterTextBoxWidget->GetText().ToString() ) )
					{
						bAnyChildFunctionPassedFilter = true;

						SignatureToTreeItemMap.Add( FunctionSymbolName, FunctionTreeItemPtr );
						FunctionList.Add( FunctionTreeItemPtr );
					}
				}

				// Add the class itself to the list if either any of its children passed the filter, or if
				// the class name passes the filter
				if( bAnyChildFunctionPassedFilter || 
					Helpers::PassesFilter( *ClassTreeItemPtr, CurrentCustomColumnMode, FilterTextBoxWidget->GetText().ToString() ) )
				{
					SignatureToTreeItemMap.Add( CurClass.Name, ClassTreeItemPtr );
					RootTreeItems.Add( ClassTreeItemPtr );
				}

				// Sort the list of functions in this class
				FunctionList.Sort( Helpers::FTreeItemSorter() );

				ClassNameToTreeItemMap.Add( CurClass.Name, FunctionList );
			}


			// NOTE: We purposely don't sort the list of classes.  We want the most derived class on top,
			// and that happens to be the order that the classes come from the code navigation engine.

			CodeTreeView->RequestTreeRefresh();

			// Expand all of the root level classes by default
			// @todo editcode: Need to avoid reseting selection and expansion state when filtering/refreshing
			const auto bShouldAlwaysExpand = true;
			const auto bShouldExpand = bShouldAlwaysExpand || IsFilterActive();		// Always expand items when searching!
			for( auto CurItemIndex = 0; CurItemIndex < RootTreeItems.Num(); ++CurItemIndex )
			{
				CodeTreeView->SetItemExpansion( RootTreeItems[ CurItemIndex ], bShouldExpand );
			}
		}
	}


	FText SCodeView::GetCustomColumnTextForTreeItem( FTreeItemPtr TreeItem ) const
	{
		FText CustomColumnText = FText::GetEmpty();

		switch( CurrentCustomColumnMode )
		{
			case ECustomColumnMode::ModuleName:
				CustomColumnText = FText::FromString(TreeItem->ModuleName);
				break;
		};

		return CustomColumnText;
	}


	TSharedRef< SWidget > SCodeView::GenerateWidgetForItemAndColumn( FTreeItemPtr Item, const FName ColumnID ) const
	{
		// Setup icon
		const FSlateBrush* IconBrush = NULL;
		FText IconToolTipText;
		if( Item->GetType() == ETreeItemType::Class )
		{
			IconBrush = FAppStyle::GetBrush( TEXT( "CodeView.ClassIcon" ) );
			IconToolTipText = LOCTEXT("ClassIconToolTip", "Class");
		}
		else if( ensure( Item->GetType() == ETreeItemType::Function ) )
		{
			IconBrush = FAppStyle::GetBrush( TEXT( "CodeView.FunctionIcon" ) );
			IconToolTipText = LOCTEXT("FunctionIconToolTip", "Function");
		}

		TSharedPtr< SHorizontalBox > TableRowContent;
		
		if( ColumnID == ColumnID_ClassesAndFunctions )
		{
			// Figure out if we should display a throbber next to this item
			bool bShowThrobber = false;
			if( Item->GetType() == ETreeItemType::Class )
			{
				const FClassTreeItem& ClassTreeItem = static_cast< FClassTreeItem& >( *Item );
				if( !ClassTreeItem.bIsCompleteList)
				{
					bShowThrobber = true;
				}
			}

			TableRowContent = 
				SNew( SHorizontalBox )
					+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding( 0.0f, 0.0f, 6.0f, 0.0f )
						[
							SNew( SImage )
								.Image( IconBrush )
								.ToolTipText( IconToolTipText )
						]
					+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew( STextBlock )

 								.Text( FText::FromString(Item->GetAsString()) ) 

								// Bind our filter text as the highlight string for the text block, so that when the user
								// starts typing search criteria, this text highlights
								.HighlightText( this, &SCodeView::GetFilterText )

								// Use the module name as the tool-tip
								.ToolTipText( FText::FromString(Item->ModuleName) )
						]
				;

			if( bShowThrobber )
			{
				TableRowContent->AddSlot()
					.AutoWidth()
					.Padding( 20.0f, 0.0f, 0.0f, 0.0f )
					[
						SNew( SThrobber )
							.PieceImage( FAppStyle::GetBrush("SmallThrobber.Chunk") )
							.NumPieces( 3 )
							.Animate( SThrobber::Opacity )
					];
			}

		}
		else if( ensure( ColumnID == ColumnID_Custom ) )
		{
			TableRowContent = 
				SNew( SHorizontalBox )

					+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding( 0.0f, 3.0f, 0.0f, 0.0f )
						[
							SNew( STextBlock )

							// Bind a delegate for custom text for this item's row
 							.Text( this, &SCodeView::GetCustomColumnTextForTreeItem, Item )

							// Bind our filter text as the highlight string for the text block, so that when the user
							// starts typing search criteria, this text highlights
							.HighlightText( this, &SCodeView::GetFilterText )

							// Use a slightly darker text color for the info column.  We want the label to stand out over this.
							.ColorAndOpacity( FLinearColor( 0.6f, 0.6f, 0.6f ) )
						]
				;
		}

		return TableRowContent.ToSharedRef();
	}

	bool SCodeView::IsReadyToPopulate() const
	{
		return bReadyToPopulate;
	}

	void SCodeView::OnDetailSectionExpansionChanged( bool bIsExpanded )
	{
		bReadyToPopulate = bIsExpanded;
		if( bReadyToPopulate )
		{
			// Refresh!
			Populate();
		}
	}


	TSharedRef< ITableRow > SCodeView::OnGenerateRowForTree( FTreeItemPtr Item, const TSharedRef< STableViewBase >& OwnerTable )
	{
		return
			SNew( SCodeViewTreeRow, OwnerTable )
				.CodeView( SharedThis( this ) )
				.Item( Item );
	}


	void SCodeView::OnGetChildrenForTree( FTreeItemPtr Parent, TArray< FTreeItemPtr >& OutChildren )
	{
		if( Parent.IsValid() )
		{
			// Only classes have child functions
			if( Parent->GetType() == ETreeItemType::Class )
			{
				// Grab all of the functions in this class
				const auto& ClassFunctions = ClassNameToTreeItemMap.FindChecked( Parent->GetAsString() );
				for( auto CurFunctionIndex = 0; CurFunctionIndex < ClassFunctions.Num(); ++CurFunctionIndex )
				{
					auto& CurFunctionTreeItem = ClassFunctions[ CurFunctionIndex ];
					OutChildren.Add( CurFunctionTreeItem );	
				}
			}
		}
	}


	void SCodeView::OnTreeSelectionChanged( FTreeItemPtr TreeItem, ESelectInfo::Type /*SelectInfo*/ )
	{
		// ...
	}

	
	void SCodeView::OnTreeDoubleClick( FTreeItemPtr TreeItem )
	{
		// Navigate to the function that was double-clicked on
		if( TreeItem->GetType() == ETreeItemType::Function )
		{
			auto const& FunctionTreeItem = StaticCastSharedPtr< FFunctionTreeItem >( TreeItem );

			// Navigate to this function (implemented in C++)!
			const bool bIgnoreLineNumber = false;
			FSourceCodeNavigation::NavigateToFunctionSourceAsync( FunctionTreeItem->FunctionSymbolName, FunctionTreeItem->ModuleName, bIgnoreLineNumber );
		}
		else
		{
			auto const& ClassTreeItem = StaticCastSharedPtr< FClassTreeItem >( TreeItem );

			if( !ClassTreeItem->AnyFunctionSymbolName.IsEmpty() )
			{
				const bool bIgnoreLineNumber = true;
				FSourceCodeNavigation::NavigateToFunctionSourceAsync( ClassTreeItem->AnyFunctionSymbolName, ClassTreeItem->ModuleName, bIgnoreLineNumber );
			}
		}
	}



	void SCodeView::OnFilterTextChanged( const FText& InFilterText )
	{
		if( bReadyToPopulate )
		{
			Populate();
		}
	}


	void SCodeView::OnFilterTextCommitted( const FText& InFilterText, ETextCommit::Type CommitInfo )
	{
		// We'll only select items if the user actually pressed the enter key.  We don't want to change
		// selection just because focus was lost from the search text field.
		if( CommitInfo == ETextCommit::OnEnter )
		{
			// Any text in the filter?  If not, we won't bother doing anything
			if( !FilterTextBoxWidget->GetText().IsEmpty() )
			{
				// @todo: Do something with all of the filtered classes/functions when the user presses enter?
			}

			#if( 0 )		// NOTE: We're experimenting with leaving the filter active after committing text
			{
				// Now that we've committed the change, clear the text field
				FilterTextBoxWidget->SetText( FText::GetEmpty() );
			}
			#endif
		}
	}


	EVisibility SCodeView::GetFilterStatusVisibility() const
	{
		return IsFilterActive() ? EVisibility::Visible : EVisibility::Collapsed;
	}


	FText SCodeView::GetFilterStatusText() const
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("TotalClassesAndFunctions"), TotalClassesAndFunctions);
		Args.Add(TEXT("VisibleClassesAndFunctions"), SignatureToTreeItemMap.Num());

		if( SignatureToTreeItemMap.Num() == 0 )
		{
			return FText::Format(LOCTEXT("ShowingNoFunctions", "No matching items ({TotalClassesAndFunctions} total)"), Args);
		}

		return FText::Format(LOCTEXT("ShowingOnlySomeFunctions", "Showing {VisibleClassesAndFunctions} of {TotalClassesAndFunctions} items"), Args);
	}

	FSlateColor SCodeView::GetFilterStatusTextColor() const
	{
		if( SignatureToTreeItemMap.Num() == 0 )
		{
			// Red = no matching items
			return FLinearColor( 1.0f, 0.4f, 0.4f );
		}
		else
		{
			// Green = found at least one match!
			return FLinearColor( 0.4f, 1.0f, 0.4f );
		}
	}


	bool SCodeView::IsFilterActive() const
	{
		return TotalClassesAndFunctions != SignatureToTreeItemMap.Num();
	}


	const FSlateBrush* SCodeView::GetFilterButtonGlyph() const
	{
		if( IsFilterActive() )
		{
			return FAppStyle::GetBrush(TEXT("SceneOutliner.FilterCancel"));
		}
		else
		{
			return FAppStyle::GetBrush(TEXT("SceneOutliner.FilterSearch"));
		}
	}


	FText SCodeView::GetFilterButtonToolTip() const
	{
		return IsFilterActive() ? LOCTEXT("ClearSearchFilter", "Clear search filter") : LOCTEXT("StartSearching", "Search");
	}

	FText SCodeView::GetFilterText() const
	{
		return FilterTextBoxWidget->GetText();
	}

	void SCodeView::OnCustomColumnModeChanged( TSharedPtr< ECustomColumnMode::Type > NewSelection, ESelectInfo::Type /*SelectInfo*/ )
	{
		CurrentCustomColumnMode = *NewSelection;

		if( bReadyToPopulate )
		{
			// Refresh and refilter the list
			Populate();
		}
	}

	
	void SCodeView::OnSymbolQueryFinished()
	{
		if( bReadyToPopulate )
		{
			// New symbols are ready, so refresh!
			Populate();
		}
	}
	
	FText SCodeView::GetSelectedModeText() const
	{
		TSharedPtr< ECustomColumnMode::Type > Selected = CustomColumnModeComboBox->GetSelectedItem();
		return (Selected.IsValid())
			? Helpers::MakeCustomColumnComboText(*Selected)
			: FText::GetEmpty();
	}

}	// namespace CodeView

#undef LOCTEXT_NAMESPACE
