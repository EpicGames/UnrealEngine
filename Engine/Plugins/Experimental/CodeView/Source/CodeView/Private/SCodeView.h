// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Widgets/SCompoundWidget.h"
#include "ActorDetailsDelegates.h"

class ITableRow;
class SSearchBox;
class STableViewBase;
namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }
template <typename ItemType> class STreeView;
template <typename OptionType> class SComboBox;

struct FTreeItem;

namespace CodeView
{
	namespace ETreeItemType
	{
		enum Type
		{
			/** C++ class */
			Class,

			/** C++ method */
			Function
		};
	};


	/** Base tree item abstract class for our code view. */
	class FTreeItem
	{
	public:
		FTreeItem()
		{
		}

		virtual ~FTreeItem()
		{
		}

		virtual ETreeItemType::Type GetType() const = 0;

		virtual const FString& GetAsString() const = 0;

		/** Module name */
		FString ModuleName;
	};


	/** Represents a C++ class in the tree view */
	class FClassTreeItem
		: public FTreeItem
	{
	public:
		virtual ETreeItemType::Type GetType() const override
		{
			return ETreeItemType::Class;
		}

		virtual const FString& GetAsString() const override
		{
			return ClassName;
		}

		virtual ~FClassTreeItem() {}

		/** Class name */
		FString ClassName;		

		/** True unless if we're still gathering data and this class' list of functions is incomplete */
		bool bIsCompleteList;

		/** Symbol name for a function in this class that will navigate us to the class' source file (ideally header file) */
		FString AnyFunctionSymbolName;
	};


	/** Represents a C++ function in the tree view */
	class FFunctionTreeItem
		: public FTreeItem
	{
	public:
		virtual ETreeItemType::Type GetType() const override
		{
			return ETreeItemType::Function;
		}

		virtual const FString& GetAsString() const override
		{
			return FunctionName;
		}

		virtual ~FFunctionTreeItem() {}

		/** Only the function name (sanitized) */
		FString FunctionName;

		/** Full symbol name */
		FString FunctionSymbolName;
	};



	typedef TSharedPtr< FTreeItem > FTreeItemPtr;


	typedef STreeView< FTreeItemPtr > SCodeTreeView;
	typedef TArray< FTreeItemPtr > FCodeTreeItems;

	/** Types of data we can display in a 'custom' tree column */
	namespace ECustomColumnMode
	{
		enum Type
		{
			/** Empty column -- doesn't display anything */
			None = 0,

			/** Module name */
			ModuleName,

			// ---

			/** Number of options */
			Count
		};
	}

	/** Slate combo box type that allows users to pick a custom column mode */
	typedef SComboBox< TSharedPtr< ECustomColumnMode::Type > > SCustomColumnModeComboBoxType;

	/**
	 * Code View widget.  Displays a hierarchical view of C++ functions in one or more classes.
	 */
	class SCodeView : public SCompoundWidget
	{

	public:

		SLATE_BEGIN_ARGS( SCodeView ){}

			SLATE_EVENT( FGetSelectedActors, GetSelectedActors )

		SLATE_END_ARGS()

		/**
		 * Construct this widget.  Called by the SNew() Slate macro.
		 *
		 * @param	InArgs	Declaration used by the SNew() macro to construct this widget
		 */
		CODEVIEW_API void Construct( const FArguments& InArgs );

		/** SCodeView destructor */
		CODEVIEW_API ~SCodeView();

		/** Called by our list to generate a widget that represents the specified item at the specified column in the tree */
		TSharedRef< SWidget > GenerateWidgetForItemAndColumn( FTreeItemPtr Item, const FName ColumnID ) const;

		/** Hooked up to the SDetailSection, so that it can notify the CodeView when it has been expanded */
		CODEVIEW_API void OnDetailSectionExpansionChanged( bool bIsExpanded );

		/** Returns true we're ready to populate the list.  This is used to set whether we should be initially expanded. */
		CODEVIEW_API bool IsReadyToPopulate() const;


	private:

		/** Populates our data set */
		void Populate();

		/** Gets text for the specified item to display in the custom column for the tree */
		FText GetCustomColumnTextForTreeItem( FTreeItemPtr TreeItem ) const;

		/** Called by STreeView to generate a table row for the specified item */
		TSharedRef< ITableRow > OnGenerateRowForTree( FTreeItemPtr Item, const TSharedRef< STableViewBase >& OwnerTable );

		/** Called by STreeView to get child items for the specified parent item */
		void OnGetChildrenForTree( FTreeItemPtr Parent, TArray< FTreeItemPtr >& OutChildren );

		/** Called by STreeView when the tree's selection has changed */
		void OnTreeSelectionChanged( FTreeItemPtr TreeItem, ESelectInfo::Type SelectInfo );

		/** Called by STreeView when the user double-clicks on an item in the tree */
		void OnTreeDoubleClick( FTreeItemPtr TreeItem );

		/**
		 * @return Returns the current filter text
		 */
		FText GetFilterText() const;

		/**
		 * Called by the editable text control when the filter text is changed by the user
		 *
		 * @param	InFilterText	The new text
		 */
		void OnFilterTextChanged( const FText& InFilterText );

		/** Called by the editable text control when a user presses enter or commits their text change */
		void OnFilterTextCommitted( const FText& InFilterText, ETextCommit::Type CommitInfo );

		/**
		 * Called by the filter button to get the image to display in the button
		 *
		 * @return	Slate brush for the button to display
		 */
		const FSlateBrush* GetFilterButtonGlyph() const;

		/** @return	The filter button tool-tip text */
		FText GetFilterButtonToolTip() const;

		/** @return	Returns whether the filter status line should be drawn */
		EVisibility GetFilterStatusVisibility() const;

		/** @return	Returns the filter status text */
		FText GetFilterStatusText() const;

		/** @return Returns color for the filter status text message, based on success of search filter */
		FSlateColor GetFilterStatusTextColor() const;

		/** @return	Returns true if the filter is currently active */
		bool IsFilterActive() const;

		/** Called by our custom column mode combo box when a new mode is selected */
		void OnCustomColumnModeChanged( TSharedPtr< ECustomColumnMode::Type > NewSelection, ESelectInfo::Type SelectInfo );

		/** Called by FSourceCodeNavigation after a symbol query finishes */
		void OnSymbolQueryFinished();

		/** Helper method to set the displayed text for the mode combo box */
		FText GetSelectedModeText() const;

	private:

		/** Our tree view */
		TSharedPtr< SCodeTreeView > CodeTreeView;

		/** Map of function signatures to list items in our data.  Used to quickly find the item for a function. */
		TMap< FString, FTreeItemPtr > SignatureToTreeItemMap;

		/** Total number of displayable tree items we've seen, before applying a search filter */
		int32 TotalClassesAndFunctions;

		/** Root level tree items */
		FCodeTreeItems RootTreeItems;

		/** Map of class names to functions in that class */
		TMap< FString, TArray< FTreeItemPtr > > ClassNameToTreeItemMap;

		/** Current custom column mode.  This is used for displaying a bit of extra data about the items, as well as
		    allowing the user to search by additional criteria */
		mutable ECustomColumnMode::Type CurrentCustomColumnMode;

		/* Widget containing the filtering text box */
		TSharedPtr< SSearchBox > FilterTextBoxWidget;

		/** Gets the list of actors that we're viewing code for */
		FGetSelectedActors GetSelectedActorsDelegate;

		/** The mode selection combo box */
		TSharedPtr< SCustomColumnModeComboBoxType > CustomColumnModeComboBox;
	};

}		// namespace CodeView
